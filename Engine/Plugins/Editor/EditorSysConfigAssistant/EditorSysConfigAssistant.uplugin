{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Editor System Configuration Assistant",
	"Description": "Editor utility for offering system configuration suggestions",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "EditorSysConfigAssistant",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}