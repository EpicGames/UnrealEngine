// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSysConfigAssistantModule.h"

#include "Features/IModularFeatures.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/NamePermissionList.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SEditorSysConfigAssistant.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#if PLATFORM_WINDOWS
#include "Windows/EditorSysConfigFeatureLastAccessTime.h"
#include "Windows/EditorSysConfigFeatureUSNJournal.h"
#endif

static const FName EditorSysConfigAssistantTabName("EditorSysConfigAssistant");

class FEditorSysConfigAssistantModule : public IEditorSysConfigAssistantModule
{
public:
	FEditorSysConfigAssistantModule() = default;
	~FEditorSysConfigAssistantModule() = default;

	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

	virtual bool CanShowSystemConfigAssistant() override;
	virtual void ShowSystemConfigAssistant() override;
private:
#if PLATFORM_WINDOWS
	FEditorSysConfigFeatureLastAccessTime LastAccessTimeFeature;
	FEditorSysConfigFeatureUSNJournal USNJournalFeature;
#endif // PLATFORM_WINDOWS
};

static TSharedRef<SDockTab> SpawnEditorSysConfigAssistantTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	DockTab->SetContent(SNew(SEditorSysConfigAssistant, DockTab, SpawnTabArgs.GetOwnerWindow()));

	return DockTab;
}

void FEditorSysConfigAssistantModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(EditorSysConfigAssistantTabName, FOnSpawnTab::CreateStatic(&SpawnEditorSysConfigAssistantTab))
		.SetDisplayName(NSLOCTEXT("EditorSysConfigAssistant", "EditorSysConfigAssistantTabTitle", "System Config"))
		.SetTooltipText(NSLOCTEXT("EditorSysConfigAssistant", "EditorSysConfigAssistantTooltipText", "Open the System Config Assistant."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());

#if PLATFORM_WINDOWS
	IModularFeatures::Get().RegisterModularFeature(LastAccessTimeFeature.GetModularFeatureName(), &LastAccessTimeFeature);
	IModularFeatures::Get().RegisterModularFeature(USNJournalFeature.GetModularFeatureName(), &USNJournalFeature);
#endif // PLATFORM_WINDOWS
}

void FEditorSysConfigAssistantModule::ShutdownModule()
{
#if PLATFORM_WINDOWS
	IModularFeatures::Get().UnregisterModularFeature(USNJournalFeature.GetModularFeatureName(), &USNJournalFeature);
	IModularFeatures::Get().UnregisterModularFeature(LastAccessTimeFeature.GetModularFeatureName(), &LastAccessTimeFeature);
#endif // PLATFORM_WINDOWS
}

bool FEditorSysConfigAssistantModule::CanShowSystemConfigAssistant()
{
	return FGlobalTabmanager::Get()->GetTabPermissionList()->PassesFilter(EditorSysConfigAssistantTabName);
}

void FEditorSysConfigAssistantModule::ShowSystemConfigAssistant()
{
	FGlobalTabmanager::Get()->TryInvokeTab(EditorSysConfigAssistantTabName);
}

IMPLEMENT_MODULE(FEditorSysConfigAssistantModule, EditorSysConfigAssistant)
