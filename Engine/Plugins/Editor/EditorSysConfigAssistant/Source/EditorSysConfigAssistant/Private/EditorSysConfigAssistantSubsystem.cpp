// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSysConfigAssistantSubsystem.h"

#include "Async/Async.h"
#include "EditorSysConfigAssistantModule.h"
#include "EditorSysConfigFeature.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "UnrealEdMisc.h"
#include "Widgets/Notifications/SNotificationList.h"

#if PLATFORM_WINDOWS
#include "Microsoft/MinimalWindowsApi.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winreg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSysConfigAssistantSubsystem)

#define LOCTEXT_NAMESPACE "EditorSysConfigAssistant"


static bool IsUsingCookedEditorContent()
{
	return FPaths::FileExists(*FPaths::Combine(FPaths::ProjectDir(), TEXT("EditorClientAssetRegistry.bin")));
}

bool UEditorSysConfigAssistantSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
#if IS_PROGRAM || !WITH_EDITOR
	return false;
#endif
	
	if (FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript || IsUsingCookedEditorContent() || !FSlateApplication::IsInitialized())
	{
		return false;
	}
	
	return Super::ShouldCreateSubsystem(Outer);
}

void UEditorSysConfigAssistantSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddUObject(this,&UEditorSysConfigAssistantSubsystem::HandleAssistantInitializationEvent);
}

void UEditorSysConfigAssistantSubsystem::Deinitialize()
{
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
	Super::Deinitialize();

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().RemoveAll(this);
	ModularFeatures.OnModularFeatureUnregistered().RemoveAll(this);
}

void UEditorSysConfigAssistantSubsystem::HandleModularFeatureRegistered(const FName& InFeatureName, IModularFeature* InFeature)
{
	if (InFeatureName == IEditorSysConfigFeature::GetModularFeatureName())
	{
		IEditorSysConfigFeature* NewSysConfigFeature = static_cast<IEditorSysConfigFeature*>(InFeature);
		NewSysConfigFeature->StartSystemCheck();
	}
}

void UEditorSysConfigAssistantSubsystem::HandleModularFeatureUnregistered(const FName& InFeatureName, IModularFeature* InFeature)
{
	if (InFeatureName == IEditorSysConfigFeature::GetModularFeatureName())
	{
		IEditorSysConfigFeature* RemovedSysConfigFeature = static_cast<IEditorSysConfigFeature*>(InFeature);
		FWriteScopeLock _(IssuesLock);
		Issues.RemoveAll([RemovedSysConfigFeature](const TSharedPtr<FEditorSysConfigIssue>& Issue)
			{
				return Issue->Feature == RemovedSysConfigFeature;
			});
	}
}

void UEditorSysConfigAssistantSubsystem::HandleAssistantInitializationEvent()
{
	if (IEditorSysConfigAssistantModule::Get().CanShowSystemConfigAssistant())
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		IModularFeatures::FScopedLockModularFeatureList LockModularFeatureList;
		TArray<IEditorSysConfigFeature*> SysConfigFeatures = IModularFeatures::Get().GetModularFeatureImplementations<IEditorSysConfigFeature>(IEditorSysConfigFeature::GetModularFeatureName());
		for (IEditorSysConfigFeature* SysConfigFeature : SysConfigFeatures)
		{
			SysConfigFeature->StartSystemCheck();
		}

		ModularFeatures.OnModularFeatureRegistered().AddUObject(this, &UEditorSysConfigAssistantSubsystem::HandleModularFeatureRegistered);
		ModularFeatures.OnModularFeatureUnregistered().AddUObject(this, &UEditorSysConfigAssistantSubsystem::HandleModularFeatureUnregistered);
	}
}

void UEditorSysConfigAssistantSubsystem::NotifySystemConfigIssues()
{
	check(IsInGameThread());
	
	UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
	if (!Subsystem)
	{
		return;
	}

	FNotificationInfo Info(LOCTEXT("SystemConfigurationIssuesExist", "System configuration is impacting editor experience"));
	Info.bUseSuccessFailIcons = true;
	Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
	Info.bFireAndForget = false;
	Info.bUseThrobber = true;
	Info.FadeOutDuration = 0.0f;
	Info.ExpireDuration = 0.0f;

	Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("OpenSysConfig", "System Config"), FText(), FSimpleDelegate::CreateLambda([]() {
		IEditorSysConfigAssistantModule::Get().ShowSystemConfigAssistant();
	}),
		SNotificationItem::ECompletionState::CS_Fail));


	Subsystem->IssueNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

	if (Subsystem->IssueNotificationItem.IsValid())
	{
		Subsystem->IssueNotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

void UEditorSysConfigAssistantSubsystem::NotifyRestart(bool bApplicationOnly)
{
	TSharedPtr<SNotificationItem> NotificationPin = RestartNotificationItem.Pin();
	if (NotificationPin.IsValid())
	{
		return;
	}
	
	FText RestartText;
	if (bApplicationOnly)
	{
		RestartText = LOCTEXT("ApplicationRestartRequiredTitle", "Application restart required to apply new system configuration");
	}
	else
	{
		RestartText = LOCTEXT("SystemRestartRequiredTitle", "System restart required to apply new system configuration");
	}
	FNotificationInfo Info(RestartText);

	// Add the buttons with text, tooltip and callback
	Info.ButtonDetails.Add(FNotificationButtonInfo(
		LOCTEXT("RestartNow", "Restart Now"), 
		LOCTEXT("RestartNowToolTip", "Restart now to finish applying your new system configuration."), 
		FSimpleDelegate::CreateUObject(this, bApplicationOnly ? &UEditorSysConfigAssistantSubsystem::OnApplicationRestartClicked : &UEditorSysConfigAssistantSubsystem::OnSystemRestartClicked))
		);
	Info.ButtonDetails.Add(FNotificationButtonInfo(
		LOCTEXT("RestartLater", "Restart Later"), 
		LOCTEXT("RestartLaterToolTip", "Dismiss this notificaton without restarting. Some new system configuration will not be applied."), 
		FSimpleDelegate::CreateUObject(this, &UEditorSysConfigAssistantSubsystem::OnRestartDismissClicked))
		);

	// We will be keeping track of this ourselves
	Info.bFireAndForget = false;

	// Set the width so that the notification doesn't resize as its text changes
	Info.WidthOverride = 300.0f;

	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;

	// Present notification
	RestartNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	NotificationPin = RestartNotificationItem.Pin();

	if (NotificationPin.IsValid())
	{
		NotificationPin->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void UEditorSysConfigAssistantSubsystem::OnApplicationRestartClicked()
{
	TSharedPtr<SNotificationItem> NotificationPin = RestartNotificationItem.Pin();
	if (NotificationPin.IsValid())
	{
		NotificationPin->SetText(LOCTEXT("RestartingNow", "Restarting..."));
		NotificationPin->SetCompletionState(SNotificationItem::CS_Success);
		NotificationPin->ExpireAndFadeout();
		RestartNotificationItem.Reset();
	}

	const bool bWarn = false;
	FUnrealEdMisc::Get().RestartEditor(bWarn);
}

void UEditorSysConfigAssistantSubsystem::OnSystemRestartClicked()
{
	TSharedPtr<SNotificationItem> NotificationPin = RestartNotificationItem.Pin();
	if (NotificationPin.IsValid())
	{
		NotificationPin->SetText(LOCTEXT("RestartingNow", "Restarting..."));
		NotificationPin->SetCompletionState(SNotificationItem::CS_Success);
		NotificationPin->ExpireAndFadeout();
		RestartNotificationItem.Reset();
	}

#if PLATFORM_WINDOWS
	CA_SUPPRESS(28159);
	::InitiateSystemShutdownEx(nullptr, nullptr, 0, false, true, SHTDN_REASON_MAJOR_OTHER | SHTDN_REASON_MINOR_MAINTENANCE | SHTDN_REASON_FLAG_PLANNED);
#endif
	RequestEngineExit(TEXT("Restarting system to apply system config changes"));

}

void UEditorSysConfigAssistantSubsystem::OnRestartDismissClicked()
{
	TSharedPtr<SNotificationItem> NotificationPin = RestartNotificationItem.Pin();
	if (NotificationPin.IsValid())
	{
		NotificationPin->SetText(LOCTEXT("RestartDismissed", "Restart Dismissed..."));
		NotificationPin->SetCompletionState(SNotificationItem::CS_None);
		NotificationPin->ExpireAndFadeout();
		RestartNotificationItem.Reset();
	}
}


void UEditorSysConfigAssistantSubsystem::AddIssue(const FEditorSysConfigIssue& Issue)
{
	FWriteScopeLock _(IssuesLock);
	const TSharedPtr<FEditorSysConfigIssue>* ExistingIssue = Issues.FindByPredicate([&Issue](const TSharedPtr<FEditorSysConfigIssue>& ExistingIssue)
		{
			return ExistingIssue->Feature == Issue.Feature;
		});

	bool bIssueAddedOrUpgraded = false;
	if (ExistingIssue)
	{
		if ((*ExistingIssue)->Severity < Issue.Severity)
		{
			**ExistingIssue = Issue;
			bIssueAddedOrUpgraded = true;
		}
	}
	else
	{
		Issues.Add(MakeShared<FEditorSysConfigIssue>(Issue));
		bIssueAddedOrUpgraded = true;
	}
	if (bIssueAddedOrUpgraded && (Issue.Severity == EEditorSysConfigIssueSeverity::High))
	{
		if (IsInGameThread())
		{
			NotifySystemConfigIssues();
		}
		else
		{
			Async(EAsyncExecution::TaskGraphMainThread, NotifySystemConfigIssues);
		}
	}
}

TArray<TSharedPtr<FEditorSysConfigIssue>> UEditorSysConfigAssistantSubsystem::GetIssues()
{
	FReadScopeLock _(IssuesLock);
	return Issues;
}

void UEditorSysConfigAssistantSubsystem::ApplySysConfigChanges(TArrayView<const TSharedPtr<FEditorSysConfigIssue>> InIssues)
{
	TArray<IEditorSysConfigFeature*> FeaturesToRecheck;
	TArray<FString> ElevatedCommands;
	EEditorSysConfigFeatureRemediationFlags AccumulatedFlags = EEditorSysConfigFeatureRemediationFlags::NoAutomatedRemediation;
	for (const TSharedPtr<FEditorSysConfigIssue>& Issue : InIssues)
	{
		EEditorSysConfigFeatureRemediationFlags Flags = Issue->Feature->GetRemediationFlags();
		if (EnumHasAnyFlags(Flags, EEditorSysConfigFeatureRemediationFlags::NoAutomatedRemediation))
		{
			continue;
		}

		AccumulatedFlags |= Flags;

		Issue->Feature->ApplySysConfigChanges(ElevatedCommands);

#if PLATFORM_WINDOWS
		if (!EnumHasAnyFlags(Flags, EEditorSysConfigFeatureRemediationFlags::RequiresSystemRestart | EEditorSysConfigFeatureRemediationFlags::RequiresApplicationRestart))
#else
		if (!EnumHasAnyFlags(Flags, EEditorSysConfigFeatureRemediationFlags::RequiresApplicationRestart))
#endif
		{
			{
				FWriteScopeLock _(IssuesLock);
				Issues.RemoveAll([Issue](const TSharedPtr<FEditorSysConfigIssue>& ExistingIssue)
					{
						return ExistingIssue->Feature == Issue->Feature;
					});
			}
			FeaturesToRecheck.Add(Issue->Feature);
		}
	}

	if (!ElevatedCommands.IsEmpty())
	{
#if PLATFORM_WINDOWS
		const TCHAR* CommandFileBaseName = TEXT("UESysConfig.bat");
#elif PLATFORM_MAC || PLATFORM_LINUX
		const TCHAR* CommandFileBaseName = TEXT("UESysConfig.sh");
#endif
		const FString CommandFilename = FPaths::Combine(FPlatformProcess::UserTempDir(), CommandFileBaseName);
		FFileHelper::SaveStringArrayToFile(ElevatedCommands, *CommandFilename);

		FString URL;
		FString Params;
#if PLATFORM_WINDOWS
		URL = TEXT("cmd.exe");
		Params = FString::Printf(TEXT("/c \"\"%s\"\""), *CommandFilename);
#elif PLATFORM_MAC || PLATFORM_LINUX
		URL = TEXT("/usr/bin/env");
		Params = FString::Printf(TEXT(" -- \"%s\""), *CommandFilename);
#endif

		int32 ExecutionReturnCode = 0;
		FPlatformProcess::ExecElevatedProcess(*URL, *Params, &ExecutionReturnCode);
	}

	for (IEditorSysConfigFeature* FeatureToRecheck : FeaturesToRecheck)
	{
		FeatureToRecheck->StartSystemCheck();
	}

#if PLATFORM_WINDOWS
	if (EnumHasAnyFlags(AccumulatedFlags, EEditorSysConfigFeatureRemediationFlags::RequiresSystemRestart))
	{
		NotifyRestart(/* bAppplicationOnly */false);
		return;
	}
#endif
	if (EnumHasAnyFlags(AccumulatedFlags, EEditorSysConfigFeatureRemediationFlags::RequiresApplicationRestart))
	{
		NotifyRestart(/* bAppplicationOnly */true);
	}
}

void UEditorSysConfigAssistantSubsystem::DismissSystemConfigNotification()
{
	if (IssueNotificationItem.IsValid())
	{
		IssueNotificationItem->ExpireAndFadeout();
	}
}

#undef LOCTEXT_NAMESPACE
