// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

struct FEditorSysConfigIssue;

/**
 * Delegate type for Editor System Configuration Assistant action running.
 *
 * The first parameter is the FEditorSysConfigIssue that you want to resolve issues on.
 */
DECLARE_DELEGATE_OneParam(FOnApplySysConfigChange, const TSharedPtr<FEditorSysConfigIssue>&)
