// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SEditorSysConfigAssistant.h"

#include "Editor/EditorEngine.h"
#include "EditorSysConfigAssistantSubsystem.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "SSimpleButton.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SEditorSysConfigAssistantIssueListView.h"
#include "Widgets/Text/STextBlock.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "SEditorSysConfigAssistant"

SEditorSysConfigAssistant::SEditorSysConfigAssistant()
{
}


SEditorSysConfigAssistant::~SEditorSysConfigAssistant()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SEditorSysConfigAssistant::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{

	ChildSlot
	[
		SNew(SVerticalBox)
			
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(0, 6))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.8f)
				.HAlign(HAlign_Fill)
				.Padding(14, 0, 0, 0)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text_Lambda([this]()
					{
						return LOCTEXT("IssuesPreamble", "The list below shows system configuration issues that are impacting editor experience.\n"
							"These are machine and operating system issues that exist outside of the editor, and addressing them can have consequences to software outside of the editor."
							" If you are uncertain about applying the changes or lack permission to make the changes, please discuss with your system administrator."
							" A button next to each item can be used to fix the configuration issue, or you can press the 'Apply All Changes' button to address all the identified issues.");
					})
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.2f)
				.HAlign(HAlign_Right)
				.Padding(0.0f, 0.0f)
				[
					SNew(SSimpleButton)
					.Text(LOCTEXT("ApplyAllChangesButton", "Apply All Changes"))
					.OnClicked_Lambda([this]()
					{
						UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
						if (!Subsystem)
						{
							return FReply::Handled();
						}

						TArray<TSharedPtr<FEditorSysConfigIssue>> IssuesCopy = Subsystem->GetIssues();
						Subsystem->ApplySysConfigChanges(IssuesCopy);
						IssueListView->RefreshIssueList();

						return FReply::Handled();
					})
					.IsEnabled_Lambda([this]()
					{
						UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
						if (!Subsystem)
						{
							return false;
						}

						return !Subsystem->GetIssues().IsEmpty();
					})
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1)
		.Padding(0)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			[
				SAssignNew(IssueListView, SEditorSysConfigAssistantIssueListView)
				.OnApplySysConfigChange(this, &SEditorSysConfigAssistant::OnApplySysConfigChange)
			]
		]
		
	];

	UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
	if (!Subsystem)
	{
		return;
	}

	Subsystem->DismissSystemConfigNotification();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SEditorSysConfigAssistant::OnApplySysConfigChange(const TSharedPtr<FEditorSysConfigIssue>& Issue)
{
	UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
	if (!Subsystem)
	{
		return;
	}

	Subsystem->ApplySysConfigChanges({Issue});
	IssueListView->RefreshIssueList();
}

#undef LOCTEXT_NAMESPACE
