// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EditorSysConfigIssue.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SEditorSysConfigAssistantIssueListView;

enum class ECheckBoxState : uint8;


/**
 * Implements a Slate widget for the editor system configuration user interface.
 */
class SEditorSysConfigAssistant
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SEditorSysConfigAssistant) { }
	SLATE_END_ARGS()

public:

	/** Constructor. */
	SEditorSysConfigAssistant();

	/** Destructor. */
	~SEditorSysConfigAssistant();

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 * @param InModel The view model to use.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);

protected:

private:
	void OnApplySysConfigChange(const TSharedPtr<FEditorSysConfigIssue>& Issue);

private:
	TSharedPtr<SEditorSysConfigAssistantIssueListView> IssueListView;

};
