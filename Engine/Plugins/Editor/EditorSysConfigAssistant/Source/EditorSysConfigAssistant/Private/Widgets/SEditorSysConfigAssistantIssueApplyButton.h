// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "SSimpleButton.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SEditorSysConfigAssistantIssueApplyButton"

/**
 * Implements a button for applying config changes to resolve a system configuration issue.
 */
class SEditorSysConfigAssistantIssueApplyButton
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SEditorSysConfigAssistantIssueApplyButton) { }
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, bool InShowText)
	{
		TSharedPtr<SVerticalBox> VerticalBoxWidget;
		ChildSlot
		[	
			SNew(SSimpleButton)
			.OnClicked(InArgs._OnClicked)
			.IsEnabled(this, &SEditorSysConfigAssistantIssueApplyButton::ButtonEnabled)
			.Icon(this, &SEditorSysConfigAssistantIssueApplyButton::GetApplyIcon)
		];

		if (InShowText && VerticalBoxWidget.IsValid())
		{
			VerticalBoxWidget->AddSlot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ApplyChangeButtonText", "Apply Change"))
			];
		}

		// Otherwise we fall back on simple text
		SetToolTipText(LOCTEXT("ApplyChangeButtonTooltip", "Apply changes to address this issue"));
	}

	virtual TSharedPtr<IToolTip> GetToolTip() override
	{
		return SWidget::GetToolTip();
	}

private:

	bool HasError() const
	{
		return false;
	}

	bool ButtonEnabled() const
	{
		return !HasError();
	}

	const FSlateBrush* GetApplyIcon() const
	{
		return HasError() ? FAppStyle::Get().GetBrush(TEXT("Icons.Error")) : FAppStyle::Get().GetBrush("Icons.Play");
	}

private:

};


#undef LOCTEXT_NAMESPACE
