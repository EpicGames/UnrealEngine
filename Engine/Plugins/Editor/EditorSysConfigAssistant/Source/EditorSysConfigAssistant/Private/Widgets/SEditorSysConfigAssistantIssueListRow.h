// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSysConfigIssue.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PlatformInfo.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/EditorSysConfigAssistantDelegates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SEditorSysConfigAssistantIssueApplyButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SEditorSysConfigAssistantIssueListRow"

/**
 * Implements a row widget for the editor's system configuration assistant issue list.
 */
class SEditorSysConfigAssistantIssueListRow
	: public STableRow<TSharedPtr<FEditorSysConfigIssue>>
{
public:

	SLATE_BEGIN_ARGS(SEditorSysConfigAssistantIssueListRow) { }
		/**
		* The Callback for when the button to resolve an issue is clicked.
		*/
		SLATE_EVENT(FOnApplySysConfigChange, OnApplySysConfigChange)
	
		/**
		 * The issue shown in this row.
		 */
		SLATE_ARGUMENT(TSharedPtr<FEditorSysConfigIssue>, Issue)

	SLATE_END_ARGS()

public:

	~SEditorSysConfigAssistantIssueListRow() = default;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow< TSharedPtr<FEditorSysConfigIssue> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(false),
			InOwnerTableView
			);

		Issue = InArgs._Issue.ToSharedRef();
		OnApplySysConfigChange = InArgs._OnApplySysConfigChange;

		TSharedRef<SUniformGridPanel> NameGrid = SNew(SUniformGridPanel).SlotPadding(FMargin(0.0f, 1.0f));
		TSharedRef<SUniformGridPanel> ValueGrid = SNew(SUniformGridPanel).SlotPadding(FMargin(0.0f, 1.0f));

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(0,0,0,1)
			[
				SNew(SBorder)
				.Padding(2)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(14, 0, 12, 0)
					[
						SNew(SBox)
						.WidthOverride(44.f)
						.HeightOverride(44.f)
						[
							SNew(SImage)
							.Image(this, &SEditorSysConfigAssistantIssueListRow::HandleIssueImage)
						]
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2,9,2,4)
						[
							SNew(STextBlock)
							.Text(this, &SEditorSysConfigAssistantIssueListRow::HandleIssueNameText)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2, 4, 2, 9)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SEditorSysConfigAssistantIssueListRow::HandleIssueDescriptionText)
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						// This Vertical box ensures the NameGrid spans only the vertical space the ValueGrid forces.
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 0, 30, 0)
							[
								NameGrid
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								ValueGrid
							]
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(12, 5, 0, 5)
					[
						SNew(SSeparator)
						.Orientation(Orient_Vertical)
						.Thickness(1.f)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(20, 0, 20, 0)
					[
						SNew(SEditorSysConfigAssistantIssueApplyButton, true)
						.OnClicked(this, &SEditorSysConfigAssistantIssueListRow::OnApplySysConfigChangeClicked)
					]
				]
			]
		];

	}

private:

	FReply OnApplySysConfigChangeClicked()
	{
		if (OnApplySysConfigChange.IsBound())
		{
			OnApplySysConfigChange.Execute(Issue);
		}

		return FReply::Handled();
	}

	// Callback for getting the icon image of the issue.
	const FSlateBrush* HandleIssueImage( ) const
	{
		return FStyleDefaults::GetNoBrush();
	}

	// Callback for getting the friendly name.
	FText HandleIssueNameText() const
	{
		return Issue->Feature->GetDisplayName();
	}

	FText HandleIssueDescriptionText() const
	{
		return Issue->Feature->GetDisplayDescription();
	}

private:

	// Holds a reference to the issue that is displayed in this row.
	TSharedPtr<FEditorSysConfigIssue> Issue;

	// Holds a delegate to be invoked when a change is applied.
	FOnApplySysConfigChange OnApplySysConfigChange;
};


#undef LOCTEXT_NAMESPACE
