// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SEditorSysConfigAssistantIssueListView.h"

#include "Editor/EditorEngine.h"
#include "EditorSysConfigAssistantSubsystem.h"
#include "EditorSysConfigFeature.h"
#include "Framework/Docking/TabManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SEditorSysConfigAssistantIssueListRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SEditorSysConfigAssistantIssueListView"

extern UNREALED_API UEditorEngine* GEditor;

SEditorSysConfigAssistantIssueListView::~SEditorSysConfigAssistantIssueListView()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SEditorSysConfigAssistantIssueListView::Construct(const FArguments& InArgs)
{
	OnApplySysConfigChange = InArgs._OnApplySysConfigChange;

	SAssignNew(IssueListView, SListView<TSharedPtr<FEditorSysConfigIssue>>)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&IssueList)
		.OnGenerateRow(this, &SEditorSysConfigAssistantIssueListView::HandleIssueListViewGenerateRow);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0)
		[
			SNew(SScrollBorder, IssueListView.ToSharedRef())
			[
				IssueListView.ToSharedRef()
			]
		]
	];

	RefreshIssueList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SEditorSysConfigAssistantIssueListView::RefreshIssueList()
{
	IssueList.Empty();

	UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
	if (!Subsystem)
	{
		return;
	}

	IssueList = Subsystem->GetIssues();
	IssueListView->RequestListRefresh();
}

bool SEditorSysConfigAssistantIssueListView::HandleIssueListRowIsEnabled(TSharedPtr<FEditorSysConfigIssue> Issue) const
{
	return true;
}

TSharedRef<ITableRow> SEditorSysConfigAssistantIssueListView::HandleIssueListViewGenerateRow(TSharedPtr<FEditorSysConfigIssue> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SEditorSysConfigAssistantIssueListRow, OwnerTable)
		.OnApplySysConfigChange(OnApplySysConfigChange)
		.Issue(InItem)
		.IsEnabled(this, &SEditorSysConfigAssistantIssueListView::HandleIssueListRowIsEnabled, InItem);
}


void SEditorSysConfigAssistantIssueListView::HandleIssueAdded(const TSharedPtr<FEditorSysConfigIssue> AddedIssue)
{
	RefreshIssueList();
}


void SEditorSysConfigAssistantIssueListView::HandleIssueRemoved(const TSharedPtr<FEditorSysConfigIssue> AddedIssue)
{
	RefreshIssueList();
}


#undef LOCTEXT_NAMESPACE
