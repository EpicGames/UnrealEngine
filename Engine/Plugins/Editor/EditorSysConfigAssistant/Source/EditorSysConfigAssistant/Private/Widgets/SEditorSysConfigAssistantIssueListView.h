// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/EditorSysConfigAssistantDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
struct FEditorSysConfigIssue;

/**
 * Implements the deployment targets panel.
 */
class SEditorSysConfigAssistantIssueListView
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SEditorSysConfigAssistantIssueListView) { }
		SLATE_EVENT(FOnApplySysConfigChange, OnApplySysConfigChange)
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SEditorSysConfigAssistantIssueListView();

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct(const FArguments& InArgs);


	/** Refresh the list of issues. */
	void RefreshIssueList();

private:

	/** Callback for getting the enabled state of a issue list row. */
	bool HandleIssueListRowIsEnabled(TSharedPtr<FEditorSysConfigIssue> Issue) const;

	/** Callback for generating a row in the issue view. */
	TSharedRef<ITableRow> HandleIssueListViewGenerateRow(TSharedPtr<FEditorSysConfigIssue> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Callback for when an issue has been added to the issue list. */
	void HandleIssueAdded(const TSharedPtr<FEditorSysConfigIssue> AddedIssue);

	/** Callback for when an issue has been added to the issue list. */
	void HandleIssueRemoved(const TSharedPtr<FEditorSysConfigIssue> RemovedIssue);

private:

	/** Holds the list of available issues. */
	TArray<TSharedPtr<FEditorSysConfigIssue>> IssueList;

	/** Holds the issue list view . */
	TSharedPtr<SListView<TSharedPtr<FEditorSysConfigIssue>> > IssueListView;

	/** Holds a delegate to be invoked when an issue applies a config change. */
	FOnApplySysConfigChange OnApplySysConfigChange;
};
