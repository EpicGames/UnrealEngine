// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSysConfigFeatureLastAccessTime.h"

#include "Async/Async.h"
#include "Editor/EditorEngine.h"
#include "EditorSysConfigAssistantSubsystem.h"
#include "Internationalization/Internationalization.h"

#if PLATFORM_WINDOWS

#include "Microsoft/MinimalWindowsApi.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winreg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

extern UNREALED_API UEditorEngine* GEditor;

FText FEditorSysConfigFeatureLastAccessTime::GetDisplayName() const
{
	return NSLOCTEXT("EditorSysConfigAssistant", "LastAccessTimeAssistantName", "Last Access Time Tracking");
}

FText FEditorSysConfigFeatureLastAccessTime::GetDisplayDescription() const
{
	return NSLOCTEXT("EditorSysConfigAssistant", "LastAccessTimeAssistantDescription",
		"Last access time tracking is enabled on your system and this can slow down asset registry scans and other operations. It is recommended that you disable last access time tracking machine-wide. Previous versions of Windows have had this disabled by default.");
}

FGuid FEditorSysConfigFeatureLastAccessTime::GetVersion() const
{
	static FGuid VersionGuid(0xe1f5a6ac, 0x86b24293, 0x8413c9af, 0xdbf49c89);
	return VersionGuid;
}

EEditorSysConfigFeatureRemediationFlags FEditorSysConfigFeatureLastAccessTime::GetRemediationFlags() const
{
	return EEditorSysConfigFeatureRemediationFlags::HasAutomatedRemediation |
		EEditorSysConfigFeatureRemediationFlags::RequiresElevation;
}

void FEditorSysConfigFeatureLastAccessTime::StartSystemCheck()
{
	Async(EAsyncExecution::TaskGraph, [this]()
		{
			UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
			if (!Subsystem)
			{
				return;
			}

			FString LastAccessUpdateValueString;
			if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SYSTEM\\CurrentControlSet\\Control\\FileSystem"), TEXT("NtfsDisableLastAccessUpdate"), LastAccessUpdateValueString))
			{
				uint32 KeyValue = 0;
				LexFromString(KeyValue, *LastAccessUpdateValueString);
				if (KeyValue & 1)
				{
					// Lowest bit being set means last access time updates are disabled
					return;
				}
			}

			FEditorSysConfigIssue NewIssue;
			NewIssue.Feature = this;
			// TODO: The severity should only be High if the asset count for the project (as obtained from the Asset Registry) is greater than some threshold
			//		 so that we're not putting notifications in front of people working on small projects.
			NewIssue.Severity = EEditorSysConfigIssueSeverity::High;
			Subsystem->AddIssue(NewIssue);
		});
}

void FEditorSysConfigFeatureLastAccessTime::ApplySysConfigChanges(TArray<FString>& OutElevatedCommands)
{
	OutElevatedCommands.Add(TEXT("fsutil behavior set disableLastAccess 1"));
}
#endif // PLATFORM_WINDOWS
