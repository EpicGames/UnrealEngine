// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_WINDOWS

#include "EditorSysConfigFeature.h"
#include "Templates/SharedPointer.h"

class FEditorSysConfigFeatureLastAccessTime : public IEditorSysConfigFeature
{
public:
	virtual ~FEditorSysConfigFeatureLastAccessTime() = default;

	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayDescription() const override;
	virtual FGuid GetVersion() const override;
	virtual EEditorSysConfigFeatureRemediationFlags GetRemediationFlags() const override;

	virtual void StartSystemCheck() override;
	virtual void ApplySysConfigChanges(TArray<FString>& OutElevatedCommands) override;
};

#endif // PLATFORM_WINDOWS