// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSysConfigFeatureUSNJournal.h"

#include "Async/Async.h"
#include "Editor/EditorEngine.h"
#include "EditorSysConfigAssistantSubsystem.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/Internationalization.h"

#if PLATFORM_WINDOWS

#include "Microsoft/MinimalWindowsApi.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winreg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

extern UNREALED_API UEditorEngine* GEditor;

const uint64 RecommendedJournalSizeBytes = 1 * 1024 * 1024 * 1024; /*1GiB*/
FText FEditorSysConfigFeatureUSNJournal::GetDisplayName() const
{
	return NSLOCTEXT("EditorSysConfigAssistant", "USNJournalAssistantName", "USN Journal Configuration");
}

FText FEditorSysConfigFeatureUSNJournal::GetDisplayDescription() const
{
	return NSLOCTEXT("EditorSysConfigAssistant", "USNJournalAssistantDescription",
		"The Update Sequence Number (USN) Journal is not configured correctly for the drive your project is stored on. This will result in slower asset discovery operations slowing down your Editor startup time. It is recommended that you create a 1 GiB USN journal on each drive you plan to store project files on.");
}

FGuid FEditorSysConfigFeatureUSNJournal::GetVersion() const
{
	static FGuid VersionGuid(0x52a34c4c, 0x93bb42c6, 0x96b5b787, 0x69b42f1c);

	return VersionGuid;
}

EEditorSysConfigFeatureRemediationFlags FEditorSysConfigFeatureUSNJournal::GetRemediationFlags() const
{
	return EEditorSysConfigFeatureRemediationFlags::HasAutomatedRemediation |
		EEditorSysConfigFeatureRemediationFlags::RequiresElevation;
}

void FEditorSysConfigFeatureUSNJournal::StartSystemCheck()
{
	Async(EAsyncExecution::TaskGraph, [this]()
		{
			UEditorSysConfigAssistantSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorSysConfigAssistantSubsystem>();
			if (!Subsystem)
			{
				return;
			}

			FString ProjectDir = FPaths::ProjectDir();
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			VolumeName = PlatformFile.FileJournalGetVolumeName(ProjectDir);
			if (VolumeName.Len() < 2 || !VolumeName.EndsWith(TEXT(":")))
			{
				// If the volume isn't a root drive such (e.g. a network path, //mount/) simply return; there is nothing more we can do.
				return;
			}

			uint64 JournalMaximumSize = PlatformFile.FileJournalGetMaximumSize(*VolumeName);	
			if (JournalMaximumSize >= RecommendedJournalSizeBytes)
			{
				// Journal exists and is at or larger than the recommended amount
				return;
			}

			FEditorSysConfigIssue NewIssue;
			NewIssue.Feature = this;
			NewIssue.Severity = EEditorSysConfigIssueSeverity::High;
			Subsystem->AddIssue(NewIssue);
		});
}

void FEditorSysConfigFeatureUSNJournal::ApplySysConfigChanges(TArray<FString>& OutElevatedCommands)
{
	OutElevatedCommands.Add(FString::Printf(TEXT("fsutil usn createjournal %s m=%" UINT64_FMT ""), *VolumeName, RecommendedJournalSizeBytes));
}
#endif // PLATFORM_WINDOWS
