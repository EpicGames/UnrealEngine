// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IEditorSysConfigAssistantModule : public IModuleInterface
{
public:
	static FORCEINLINE IEditorSysConfigAssistantModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IEditorSysConfigAssistantModule>("EditorSysConfigAssistant");
	}

	/** Checks if it is possible to show the system configuration assistant UI to the user */
	virtual bool CanShowSystemConfigAssistant() = 0;

	/** Attempts to present the system configuration assistant UI to the user */
	virtual void ShowSystemConfigAssistant() = 0;
};

