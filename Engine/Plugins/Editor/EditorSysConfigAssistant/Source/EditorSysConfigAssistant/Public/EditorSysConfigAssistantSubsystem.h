// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "EditorSysConfigIssue.h"
#include "Templates/SharedPointer.h"

#include "EditorSysConfigAssistantSubsystem.generated.h"

#define UE_API EDITORSYSCONFIGASSISTANT_API

class IModularFeature;
class SNotificationItem;

UCLASS(MinimalAPI)
class UEditorSysConfigAssistantSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

	/** Can be called on any thread */
	UE_API void AddIssue(const FEditorSysConfigIssue& Issue);
	/** Can be called on any thread */
	UE_API TArray<TSharedPtr<FEditorSysConfigIssue>> GetIssues();

	/** Must be called on the game thread */
	UE_API void ApplySysConfigChanges(TArrayView<const TSharedPtr<FEditorSysConfigIssue>> Issues);

	/** Must be called on the game thread */
	UE_API void DismissSystemConfigNotification();

private:
	UE_API void HandleModularFeatureRegistered(const FName& InFeatureName, IModularFeature* InFeature);
	UE_API void HandleModularFeatureUnregistered(const FName& InFeatureName, IModularFeature* InFeature);
	
	UE_API void HandleAssistantInitializationEvent();

	/** Must be called on the game thread */
	static UE_API void NotifySystemConfigIssues();

	/** Must be called on the game thread */
	UE_API void NotifyRestart(bool bApplicationOnly);

	UE_API void OnApplicationRestartClicked();
	UE_API void OnSystemRestartClicked();
	UE_API void OnRestartDismissClicked();

	FRWLock IssuesLock;
	TArray<TSharedPtr<FEditorSysConfigIssue>> Issues;

	TSharedPtr<SNotificationItem> IssueNotificationItem;
	TWeakPtr<SNotificationItem> RestartNotificationItem;
};

#undef UE_API
