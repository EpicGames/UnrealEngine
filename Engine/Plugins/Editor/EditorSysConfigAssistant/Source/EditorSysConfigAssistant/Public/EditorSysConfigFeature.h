// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"


enum class EEditorSysConfigFeatureRemediationFlags : uint32
{
	NoAutomatedRemediation		= 0,
	HasAutomatedRemediation		= 1 << 0,
	RequiresElevation			= 1 << 1,
	RequiresApplicationRestart	= 1 << 2,
#if PLATFORM_WINDOWS
	// Windows specific support for system restart
	RequiresSystemRestart		= 1 << 3,
#endif
};
ENUM_CLASS_FLAGS(EEditorSysConfigFeatureRemediationFlags);

class IEditorSysConfigFeature : public IModularFeature
{
public:
	virtual ~IEditorSysConfigFeature() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("EditorSysConfigFeature"));
		return FeatureName;
	}

	virtual FText GetDisplayName() const = 0;
	virtual FText GetDisplayDescription() const = 0;
	virtual FGuid GetVersion() const = 0;
	virtual EEditorSysConfigFeatureRemediationFlags GetRemediationFlags() const = 0;
	
	/** Called from the game thread, encouraged to perform work asynchronously */
	virtual void StartSystemCheck() = 0;

	/** Called from the game thread, must complete synchronously */
	virtual void ApplySysConfigChanges(TArray<FString>& OutElevatedCommands) = 0;
};
