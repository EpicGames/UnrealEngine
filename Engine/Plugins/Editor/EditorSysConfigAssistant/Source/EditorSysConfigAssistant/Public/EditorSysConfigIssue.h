// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSysConfigFeature.h"

enum class EEditorSysConfigIssueSeverity : uint8
{
	High,
	Medium,
	Low,
};

struct FEditorSysConfigIssue
{
	IEditorSysConfigFeature* Feature;
	EEditorSysConfigIssueSeverity Severity;
};
